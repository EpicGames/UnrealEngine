// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealInsightsMain.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSCommandLineHelper.h"
#include "IOS/SlateOpenGLESView.h"
#include "Widgets/Testing/STestSuite.h"

#import <UIKit/UIKit.h>


#define IOS_MAX_PATH 1024
#define CMD_LINE_MAX 16384


FString GSavedCommandLine;


void FAppEntry::Suspend()
{
}


void FAppEntry::Resume()
{
}


void FAppEntry::SuspendTick()
{
}


bool FAppEntry::IsStartupMoviePlaying()
{
	return false;
}


void FAppEntry::PreInit(IOSAppDelegate* AppDelegate, UIApplication* Application)
{
	// make a controller object
	AppDelegate.SlateController = [[SlateOpenGLESViewController alloc] init];

	// property owns it now
	[AppDelegate.SlateController release];

	// point to the GL view we want to use
	AppDelegate.RootView = [AppDelegate.SlateController view];

	if (AppDelegate.OSVersion >= 6.0f)
	{
		// this probably works back to OS4, but would need testing
		[AppDelegate.Window setRootViewController:AppDelegate.SlateController];
	}
	else
	{
		[AppDelegate.Window addSubview:AppDelegate.RootView];
	}
}


void FAppEntry::PlatformInit()
{
}


void FAppEntry::Init()
{
	// Start up the main loop.
	GEngineLoop.PreInit(FCommandLine::Get());

	// Move it to this thread.
	SlateOpenGLESView* View = (SlateOpenGLESView*)[IOSAppDelegate GetDelegate].RootView;
	[EAGLContext setCurrentContext:View.Context];

	// Crank up a normal Slate application using the platform's standalone renderer.
	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());

	// Bring up the UI.
	{
		//TODO: RestoreUI();
	}

#if WITH_SHARED_POINTER_TESTS
	SharedPointerTesting::TestSharedPointer<ESPMode::NotThreadSafe>();
	SharedPointerTesting::TestSharedPointer<ESPMode::ThreadSafe>();
#endif

	// Loop while the server does the rest.
	double LastTime = FPlatformTime::Seconds();
}


void FAppEntry::Tick()
{
	FSlateApplication::Get().PumpMessages();
	FSlateApplication::Get().Tick();

	// Sleep
	FPlatformProcess::Sleep(0);
}


void FAppEntry::Shutdown()
{
	FSlateApplication::Shutdown();
}


int main(int argc, char *argv[])
{
	for (int32 Option = 1; Option < argc; Option++)
	{
		GSavedCommandLine += TEXT(" ");
		GSavedCommandLine += ANSI_TO_TCHAR(argv[Option]);
	}

	FIOSCommandLineHelper::InitCommandArgs(FString());

	@autoreleasepool {
		return UIApplicationMain(argc, argv, nil, NSStringFromClass([IOSAppDelegate class]));
	}
}
