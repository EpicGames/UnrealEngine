// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealInsightsMain.h"
#include "Mac/MacProgramDelegate.h"
#include "LaunchEngineLoop.h"

@interface InsightsAppDelegate : MacProgramDelegate
{
	bool bHasFinishedLaunching;
}

@end


@implementation InsightsAppDelegate

- (void)awakeFromNib
{
	bHasFinishedLaunching = false;
}

- (BOOL)application : (NSApplication*)theApplication openFile : (NSString*)filename
{
	FString StringFilename(filename);
	if(!StringFilename.EndsWith(TEXT(".utrace")))
	{
		return NO;
	}
	if (!bHasFinishedLaunching)
	{
		if ([[NSFileManager defaultManager]fileExistsAtPath:filename] )
		{
			SavedCommandLine = SavedCommandLine + TEXT(" ") + StringFilename;
			return YES;
		}
		return NO;
	}
	else if ([[NSFileManager defaultManager]fileExistsAtPath:filename] )
	{
		NSURL* BundleURL = [[NSRunningApplication currentApplication] bundleURL];

		NSWorkspaceOpenConfiguration* Configuration = [NSWorkspaceOpenConfiguration configuration];
		[Configuration setCreatesNewApplicationInstance:YES];
		[Configuration setPromptsUserIfNeeded:YES];
		[Configuration setArguments:[NSArray arrayWithObject: filename]];

		[[NSWorkspace sharedWorkspace]
			openApplicationAtURL: BundleURL
			configuration: Configuration
			completionHandler:^(NSRunningApplication * _Nullable app, NSError * _Nullable error)
			{
				if (error) {
					NSLog(@"Failed to run the app: %@", error.localizedDescription);
				}
			}
		];

		return YES;
	}
	else
	{
		return NO;
	}
}

- (void)applicationDidFinishLaunching:(NSNotification *)Notification
{
	bHasFinishedLaunching = true;
	
	[super applicationDidFinishLaunching:Notification];
}

@end


int main(int argc, char *argv[])
{
	// make custom delegate
	InsightsAppDelegate* Delegate = [[InsightsAppDelegate alloc] initWithProgramMain:UnrealInsightsMain programExit:FEngineLoop::AppExit];
	// run with it
	return [MacProgramDelegate mainWithArgc:argc argv:argv existingDelegate:Delegate];
}
