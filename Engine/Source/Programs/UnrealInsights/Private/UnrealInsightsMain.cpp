// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealInsightsMain.h"

#include "CoreGlobals.h"
#include "RequiredProgramMainCPPInclude.h"
#include "UserInterfaceCommand.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#if PLATFORM_UNIX
#include <sys/file.h>
#include <errno.h>
#endif

IMPLEMENT_APPLICATION(UnrealInsights, "UnrealInsights");

////////////////////////////////////////////////////////////////////////////////////////////////////

FString GetTraceFileFromCmdLine(const TCHAR* CommandLine)
{
	const uint32 MaxPath = FPlatformMisc::GetMaxPathLength();
	TCHAR* FilenameBuffer = new TCHAR[MaxPath + 1];
	FilenameBuffer[0] = 0;

	// Try getting the trace file from the -OpenTraceFile= parameter first.
	if (FParse::Value(CommandLine, TEXT("-OpenTraceFile="), FilenameBuffer, MaxPath, true))
	{
		FString TraceFile(FilenameBuffer);
		delete[] FilenameBuffer;
		return TraceFile;
	}

	// Support for opening a trace file by double clicking a .utrace file.
	// In this case, the app will receive as the first parameter a utrace file path.

	const TCHAR* Str = CommandLine;
	if (FParse::Token(Str, FilenameBuffer, MaxPath, false))
	{
		FString TraceFile(FilenameBuffer);
		if (TraceFile.EndsWith(TEXT(".utrace")))
		{
			delete[] FilenameBuffer;
			return TraceFile;
		}
	}

	delete[] FilenameBuffer;
	return FString();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool CheckFrontendSingleInstance()
{
#if PLATFORM_WINDOWS
	// Create a named event that other processes can detect.
	// It allows only a single instance of Unreal Insights Frontend.
	HANDLE SessionBrowserEvent = CreateEvent(NULL, true, false, TEXT("Local\\UnrealInsightsBrowser"));
	if (SessionBrowserEvent == NULL || GetLastError() == ERROR_ALREADY_EXISTS)
	{
		// Another Session Browser process is already running.

		if (SessionBrowserEvent != NULL)
		{
			CloseHandle(SessionBrowserEvent);
		}

		// Activate the respective window.
		HWND Window = FindWindowW(0, L"Unreal Insights Frontend");
		if (Window)
		{
			ShowWindow(Window, SW_SHOW);
			SetForegroundWindow(Window);

			FLASHWINFO FlashInfo;
			FlashInfo.cbSize = sizeof(FLASHWINFO);
			FlashInfo.hwnd = Window;
			FlashInfo.dwFlags = FLASHW_ALL;
			FlashInfo.uCount = 3;
			FlashInfo.dwTimeout = 0;
			FlashWindowEx(&FlashInfo);
		}

		return false;
	}
#endif // PLATFORM_WINDOWS

#if PLATFORM_UNIX
	int FileHandle = open("/var/run/UnrealInsightsBrowser.pid", O_CREAT | O_RDWR, 0666);
	int Ret = flock(FileHandle, LOCK_EX | LOCK_NB);
	if (Ret && EWOULDBLOCK == errno)
	{
		// Another Session Browser process is already running.

		// Activate the respective window.
		//TODO: "wmctrl -a Insights"

		return false;
	}
#endif

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Platform agnostic implementation of the main entry point.
 */
int32 UnrealInsightsMain(const TCHAR* CommandLine)
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

	// Override the stack size for the thread pool.
	FQueuedThreadPool::OverrideStackSize = 256 * 1024;

	bool bFrontendMode;
	FString TraceFileToOpen;

	if (FCString::Strifind(CommandLine, TEXT("-OpenTraceId=")) ||
		FCString::Strifind(CommandLine, TEXT("-ListenForDirectTrace")) ||
		FCString::Strifind(CommandLine, TEXT("-TraceSecurePort=")))
	{
		bFrontendMode = false;
	}
	else
	{
		TraceFileToOpen = GetTraceFileFromCmdLine(CommandLine);
		bFrontendMode = TraceFileToOpen.IsEmpty();

		// Only a single instance of Unreal Insights Frontend window/process is allowed.
		if (bFrontendMode && !CheckFrontendSingleInstance())
		{
			return 0;
		}
	}

	FString NewCommandLine = CommandLine;

	// Add -Messaging if it was not given in the command line.
	if (!FParse::Param(*NewCommandLine, TEXT("Messaging")))
	{
		NewCommandLine += TEXT(" -Messaging");
	}

	// Initialize core.
	GEngineLoop.PreInit(*NewCommandLine);

	// Tell the module manager it may now process newly-loaded UObjects when new C++ modules are loaded.
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	FUserInterfaceCommand::Run(bFrontendMode, TraceFileToOpen);

	// Shut down.
	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();

	return 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
