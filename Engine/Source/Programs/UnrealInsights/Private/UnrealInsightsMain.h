// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * The application's main function.
 *
 * @param CommandLine The application command line.
 * @return Application's exit value.
 */
int32 UnrealInsightsMain(const TCHAR* CommandLine);
