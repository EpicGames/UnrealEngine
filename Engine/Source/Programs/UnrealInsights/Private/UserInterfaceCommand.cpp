// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterfaceCommand.h"

#include "Async/TaskGraphInterfaces.h"
#include "Containers/AnsiString.h"
#include "Containers/Ticker.h"
#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformProcess.h"
#include "ISlateReflectorModule.h"
#include "ISourceCodeAccessModule.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Base64.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"
#include "StandaloneRenderer.h"
#include "Stats/StatsSystem.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"

// TraceInsightsCore
#include "InsightsCore/Version.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"

// TraceInsightsFrontend
#include "InsightsFrontend/ITraceInsightsFrontendModule.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define IDEAL_FRAMERATE 60
#define BACKGROUND_FRAMERATE 4
#define IDLE_INPUT_SECONDS 5.0f

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UserInterfaceCommand
{
	const TCHAR* EngineExitInvalidArgsReason = TEXT("Invalid arguments");
	TSharedRef<FWorkspaceItem> DeveloperTools = FWorkspaceItem::NewGroup(NSLOCTEXT("UnrealInsights", "DeveloperToolsMenu", "Developer Tools"));

	bool IsApplicationBackground()
	{
		return !FPlatformApplicationMisc::IsThisApplicationForeground() && (FPlatformTime::Seconds() - FSlateApplication::Get().GetLastUserInteractionTime()) > IDLE_INPUT_SECONDS;
	}

	void AdaptiveSleep(float Seconds)
	{
		const double IdealFrameTime = 1.0 / IDEAL_FRAMERATE;
		if (Seconds > IdealFrameTime)
		{
			// While in background, pump message at ideal frame time and get out of background as soon as input is received
			const double WakeupTime = FPlatformTime::Seconds() + Seconds;
			while (IsApplicationBackground() && FPlatformTime::Seconds() < WakeupTime)
			{
				FSlateApplication::Get().PumpMessages();
				FPlatformProcess::Sleep((float)FMath::Clamp(WakeupTime - FPlatformTime::Seconds(), 0.0, IdealFrameTime));
			}
		}
		else
		{
			FPlatformProcess::Sleep(Seconds);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserInterfaceCommand::Run(bool bFrontendMode, const FString& TraceFileToOpen)
{
	// Crank up a normal Slate application using the platform's standalone renderer.
	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());

#if WITH_LIVE_CODING
	FPlatformMisc::SetUBTTargetName(TEXT("UnrealInsights"));
	FModuleManager::Get().LoadModule("LiveCoding");
#endif

	// Load required modules.
	FModuleManager::Get().LoadModuleChecked("TraceInsightsCore");
	if (bFrontendMode)
	{
		FModuleManager::Get().LoadModuleChecked("TraceInsightsFrontend");
	}
	else
	{
		FModuleManager::Get().LoadModuleChecked("TraceInsights");
	}

	// Load plug-ins.
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault);
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::Default);

	// Load optional modules.
	if (FModuleManager::Get().ModuleExists(TEXT("SettingsEditor")))
	{
		FModuleManager::Get().LoadModule("SettingsEditor");
	}

	InitializeSlateApplication(bFrontendMode, TraceFileToOpen);

	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostDefault);

	//////////////////////////////////////////////////
	// Initialize source code access.

	// Load the source code access module.
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(FName("SourceCodeAccess"));

	// Manually load in the source code access plugins, as standalone programs don't currently support plugins.
#if PLATFORM_MAC
	IModuleInterface& XCodeSourceCodeAccessModule = FModuleManager::LoadModuleChecked<IModuleInterface>(FName("XCodeSourceCodeAccess"));
	SourceCodeAccessModule.SetAccessor(FName("XCodeSourceCodeAccess"));
#elif PLATFORM_WINDOWS
	IModuleInterface& VisualStudioSourceCodeAccessModule = FModuleManager::LoadModuleChecked<IModuleInterface>(FName("VisualStudioSourceCodeAccess"));
	SourceCodeAccessModule.SetAccessor(FName("VisualStudioSourceCodeAccess"));
#endif

	//////////////////////////////////////////////////

#if WITH_SHARED_POINTER_TESTS
	SharedPointerTesting::TestSharedPointer<ESPMode::NotThreadSafe>();
	SharedPointerTesting::TestSharedPointer<ESPMode::ThreadSafe>();
#endif

	const bool bDisableFramerateThrottle = FParse::Param(FCommandLine::Get(), TEXT("DisableFramerateThrottle"));

	// Enter main loop.
	double DeltaTime = 0.0;
	double LastTime = FPlatformTime::Seconds();
	const float IdealFrameTime = 1.0f / IDEAL_FRAMERATE;
	const float BackgroundFrameTime = 1.0f / BACKGROUND_FRAMERATE;

	while (!IsEngineExitRequested())
	{
		// Save the state of the tabs here rather than after close of application (the tabs are undesirably saved out with ClosedTab state on application close).
		//UserInterfaceCommand::UserConfiguredNewLayout = FGlobalTabmanager::Get()->PersistLayout();

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		FSlateApplication::Get().PumpMessages();
		FSlateApplication::Get().Tick();
		FTSTicker::GetCoreTicker().Tick(static_cast<float>(DeltaTime));

		// Throttle frame rate.
		const float FrameTime = !bDisableFramerateThrottle && UserInterfaceCommand::IsApplicationBackground() ? BackgroundFrameTime : IdealFrameTime;

		UserInterfaceCommand::AdaptiveSleep(FMath::Max<float>(0.0f, FrameTime - static_cast<float>(FPlatformTime::Seconds() - LastTime)));

		double CurrentTime = FPlatformTime::Seconds();
		DeltaTime =  CurrentTime - LastTime;
		LastTime = CurrentTime;

		UE::Stats::FStats::AdvanceFrame(false);

		FCoreDelegates::OnEndFrame.Broadcast();
		GLog->FlushThreadedLogs();

		GFrameCounter++;
	}

	ShutdownSlateApplication();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserInterfaceCommand::InitializeSlateApplication(bool bFrontendMode, const FString& TraceFileToOpen)
{
	FSlateApplication::InitHighDPI(true);

	//const FSlateBrush* AppIcon = new FSlateImageBrush(FPaths::EngineContentDir() / "Editor/Slate/Icons/Insights/AppIcon_24x.png", FVector2D(24.0f, 24.0f));
	//FSlateApplication::Get().SetAppIcon(AppIcon);

	// Set the application name.
	const FText ApplicationTitle = FText::Format(NSLOCTEXT("UnrealInsights", "AppTitle", "Unreal Insights {0}"), FText::FromString(TEXT(UNREAL_INSIGHTS_VERSION_STRING_EX)));
	FGlobalTabmanager::Get()->SetApplicationTitle(ApplicationTitle);

	// Load widget reflector.
	const bool bAllowDebugTools = FParse::Param(FCommandLine::Get(), TEXT("DebugTools"));
	if (bAllowDebugTools)
	{
		FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector").RegisterTabSpawner(UserInterfaceCommand::DeveloperTools);
	}

	//////////////////////////////////////////////////

	FString StoreHost = TEXT("127.0.0.1");
	uint32 StorePort = 0;
	bool bUseCustomStoreAddress = false;

	if (FParse::Value(FCommandLine::Get(), TEXT("-Store="), StoreHost, true))
	{
		int32 Index = INDEX_NONE;
		if (StoreHost.FindChar(TEXT(':'), Index))
		{
			StorePort = FCString::Atoi(*StoreHost.RightChop(Index + 1));
			StoreHost.LeftInline(Index);
		}
		bUseCustomStoreAddress = true;
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-StoreHost="), StoreHost, true))
	{
		bUseCustomStoreAddress = true;
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-StorePort="), StorePort))
	{
		bUseCustomStoreAddress = true;
	}

	//////////////////////////////////////////////////

	if (!bFrontendMode) // viewer mode
	{
		FModuleManager::Get().LoadModuleChecked("TraceInsights");
		IUnrealInsightsModule& TraceInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

		// This parameter will cause the application to close when analysis fails to start or completes successfully.
		const bool bAutoQuit = FParse::Param(FCommandLine::Get(), TEXT("AutoQuit"));

		// This parameter will prevent the application from closing until analysis is completed and symbols have been resolved. Must be used with the -AutoQuit flag to work.
		const bool bWaitForSymbolResolver = FParse::Param(FCommandLine::Get(), TEXT("WaitForSymbolResolver"));

		const bool bInitializeTesting = FParse::Param(FCommandLine::Get(), TEXT("InsightsTest"));
		if (bInitializeTesting)
		{
			const bool bInitAutomationModules = true;
			TraceInsightsModule.InitializeTesting(bInitAutomationModules, bAutoQuit);
		}

		uint32 TraceId = 0;
		FString TraceIdString;
		bool bUseTraceId = FParse::Value(FCommandLine::Get(), TEXT("-OpenTraceId="), TraceIdString);
		if (bUseTraceId)
		{
			if (TraceIdString.StartsWith(TEXT("0x")))
			{
				TCHAR* End;
				TraceId = FCString::Strtoi(*TraceIdString + 2, &End, 16);
			}
			else
			{
				TCHAR* End;
				TraceId = FCString::Strtoi(*TraceIdString, &End, 10);
			}
		}

		const bool bListenForDirectTrace = FParse::Param(FCommandLine::Get(), TEXT("ListenForDirectTrace"));
		uint16 DirectTracePort = 0;
		if (bListenForDirectTrace)
		{
			FParse::Value(FCommandLine::Get(), TEXT("-DirectTracePort="), DirectTracePort);
		}

		uint16 SecurePort = 0;
		const bool bConnectToSecureTrace = FParse::Value(FCommandLine::Get(), TEXT("-TraceSecurePort="), SecurePort);

		FString Cmd;
		if (FParse::Value(FCommandLine::Get(), TEXT("-ExecOnAnalysisCompleteCmd="), Cmd, false))
		{
			TraceInsightsModule.ScheduleCommand(Cmd);
		}

		const bool bNoUI = FParse::Param(FCommandLine::Get(), TEXT("NoUI"));
		if (!bNoUI)
		{
			TraceInsightsModule.CreateSessionViewer(bAllowDebugTools);
		}

		TraceInsightsModule.SetAutoQuit(bAutoQuit, bWaitForSymbolResolver);
		if (bConnectToSecureTrace)
		{
			FString SecureAuthBase64, SecureCertBase64, SecureHost(TEXT("localhost"));
			FString SecureCertPath, SecureIdentity;
			TArray<uint8> SecureAuthBin, SecureCertBin;

			FParse::Value(FCommandLine::Get(), TEXT("-TraceSecureHost="), SecureHost);
			FParse::Value(FCommandLine::Get(), TEXT("-TraceSecureAuth="), SecureAuthBase64);
			FParse::Value(FCommandLine::Get(), TEXT("-TraceSecureCertPath="), SecureCertPath);
			FParse::Value(FCommandLine::Get(), TEXT("-TraceSecureCert="), SecureCertBase64);
			FParse::Value(FCommandLine::Get(), TEXT("-TraceSecureIdentity="), SecureIdentity);

			FAnsiString SecureIdentityAnsi(SecureIdentity);

			if (!SecureAuthBase64.IsEmpty() && !FBase64::Decode(SecureAuthBase64, SecureAuthBin))
			{
				UE_LOGF(LogInit, Error, "Failed to decode secure authorization token.");
				RequestEngineExit(UserInterfaceCommand::EngineExitInvalidArgsReason);
				return;
			}

			if (!SecureCertBase64.IsEmpty() && !FBase64::Decode(SecureCertBase64, SecureCertBin))
			{
				UE_LOGF(LogInit, Error, "Failed to decode secure certificate.");
				RequestEngineExit(UserInterfaceCommand::EngineExitInvalidArgsReason);
				return;
			}
			
			TraceInsightsModule.StartAnalysisWithSecureTrace(
				*SecureHost, 
				SecurePort, 
				SecureAuthBin, 
				*SecureCertPath,
				SecureCertBin,
				SecureIdentityAnsi
			);
		}
		else if (bListenForDirectTrace)
		{
			TraceInsightsModule.StartAnalysisWithDirectTrace(nullptr, DirectTracePort);
		}
		else if (bUseTraceId)
		{
			TraceInsightsModule.ConnectToStore(*StoreHost, StorePort);
			TraceInsightsModule.StartAnalysisForTrace(TraceId);
		}
		else
		{
			TraceInsightsModule.StartAnalysisForTraceFile(*TraceFileToOpen);
		}
	}
	else // frontend mode
	{
		FModuleManager::Get().LoadModuleChecked("TraceInsightsFrontend");
		ITraceInsightsFrontendModule& TraceInsightsFrontendModule = FModuleManager::LoadModuleChecked<ITraceInsightsFrontendModule>("TraceInsightsFrontend");

		// Ensure target platform manager is referenced early as it must be created on the main thread.
		FModuleManager::Get().LoadModuleChecked("DesktopPlatform");
		FConfigCacheIni::InitializeConfigSystem();
		GetTargetPlatformManager();

		FModuleManager::Get().LoadModuleChecked("Messaging");
		FModuleManager::Get().LoadModuleChecked("OutputLog");

		// Load optional modules.
		FModuleManager::Get().LoadModule("DeviceManager");
		FModuleManager::Get().LoadModule("SessionFrontend");

		FString AutomationTests;
		bool bRunAutomationTests =
			FParse::Value(FCommandLine::Get(), TEXT("-ExecBrowserAutomationTest="), AutomationTests, false) ||
			FParse::Value(FCommandLine::Get(), TEXT("-RunAutomationTests="), AutomationTests, false);

		TraceInsightsFrontendModule.ConnectToStore(*StoreHost, StorePort);

		UE::Insights::FCreateFrontendWindowParams Params;
		Params.bAllowDebugTools = bAllowDebugTools;
		Params.bInitializeTesting = FParse::Param(FCommandLine::Get(), TEXT("InsightsTest"));
		Params.bStartProcessWithStompMalloc = FParse::Param(FCommandLine::Get(), TEXT("stompmalloc"));
		Params.bDisableFramerateThrottle = FParse::Param(FCommandLine::Get(), TEXT("DisableFramerateThrottle"));
		Params.bAutoQuit = FParse::Param(FCommandLine::Get(), TEXT("AutoQuit"));
		TraceInsightsFrontendModule.CreateFrontendWindow(Params);

		if (bRunAutomationTests)
		{
			TraceInsightsFrontendModule.RunAutomationTests(AutomationTests);
		}
	}
	UE_LOGF(LogInit, Display, "Insights slate application initialized successfully.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserInterfaceCommand::ShutdownSlateApplication()
{
	IUnrealInsightsModule& TraceInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	TraceInsightsModule.ShutdownUserInterface();

	// Shut down application.
	FSlateApplication::Shutdown();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
