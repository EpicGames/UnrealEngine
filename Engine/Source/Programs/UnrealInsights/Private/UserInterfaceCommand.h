// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FUserInterfaceCommand
{
public:
	/** Executes the command. */
	static void Run(bool bFrontendMode, const FString& TraceFileToOpen);

protected:
	/**
	 * Initializes the Slate application.
	 */
	static void InitializeSlateApplication(bool bFrontendMode, const FString& TraceFileToOpen);

	/**
	 * Shuts down the Slate application.
	 */
	static void ShutdownSlateApplication();
};
