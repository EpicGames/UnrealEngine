// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealInsightsMain.h"
#include "HAL/ExceptionHandling.h"
#include "Windows/WindowsHWrapper.h"
#include "LaunchEngineLoop.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceError.h"

/**
 * The main application entry point for Windows platforms.
 *
 * @param hInInstance Handle to the current instance of the application.
 * @param hPrevInstance Handle to the previous instance of the application (always NULL).
 * @param lpCmdLine Command line for the application.
 * @param nShowCmd Specifies how the window is to be shown.
 * @return Application's exit value.
 */
int WINAPI WinMain(_In_ HINSTANCE hInInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ LPSTR lpCmdLine, _In_ int nShowCmd)
{
	hInstance = hInInstance;

	const TCHAR* CmdLine = ::GetCommandLineW();
	CmdLine = FCommandLine::RemoveExeName(CmdLine);

#if !UE_BUILD_SHIPPING
	if (FParse::Param(CmdLine, TEXT("crashreports")))
	{
		GAlwaysReportCrash = true;
	}
#endif

	int32 ErrorLevel = 0;

#if UE_BUILD_DEBUG
	if (!GAlwaysReportCrash)
#else
	if (FPlatformMisc::IsDebuggerPresent() && !GAlwaysReportCrash)
#endif
	{
		ErrorLevel = UnrealInsightsMain(CmdLine);
	}
	else
	{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__try
#endif
		{
			GIsGuarded = 1;
			ErrorLevel = UnrealInsightsMain(CmdLine);
			GIsGuarded = 0;
		}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__except (ReportCrash(GetExceptionInformation()))
		{
			ErrorLevel = 1;
			GError->HandleError();
			FPlatformMisc::RequestExit(true);
		}
#endif
	}

	FEngineLoop::AppExit();

	return ErrorLevel;
}
