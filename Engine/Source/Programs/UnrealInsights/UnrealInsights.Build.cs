// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealInsights : ModuleRules
{
	public UnrealInsights(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		bRequiresPlatformSDK = true;

		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework",
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"Projects",
				"Slate",
				"SlateCore",
				"SourceCodeAccess",
				"StandaloneRenderer",
				"TargetPlatform",
				"TraceInsights",
				"TraceInsightsCore",
				"TraceInsightsFrontend",
			}
		);

		// For Session Frontend and Message Bus
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"OutputLog",
				"SessionFrontend",
				"TargetDeviceServices",
				"TcpMessaging",
				"UdpMessaging",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"SlateReflector"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"SlateReflector"
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PrivateDependencyModuleNames.Add("XCodeSourceCodeAccess");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "CEF3");
		}
		else if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.Add("VisualStudioSourceCodeAccess");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PrivateDependencyModuleNames.AddRange(
				new string [] {
					"NetworkFile",
					"StreamingFile"
				}
			);
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnixCommonStartup"
				}
			);
		}

		if (Target.bWithLiveCoding)
		{
			PrivateDependencyModuleNames.Add("LiveCoding");
		}
	}
}
