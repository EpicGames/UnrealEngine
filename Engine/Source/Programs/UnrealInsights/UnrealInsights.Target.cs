// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;
using EpicGames.Core;

[SupportedPlatforms("Win64", "Linux", "Mac")]
public class UnrealInsightsTarget : TargetRules
{
	[CommandLine("-Monolithic")]
	public bool bMonolithic = false;

	[CommandLine("-Trace")]
	public bool bCmdLineTrace = false;

	[CommandLine("-TraceAnalysisDebug")]
	public bool bTraceAnalysisDebug = false;

	public UnrealInsightsTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = bMonolithic ? TargetLinkType.Monolithic : TargetLinkType.Modular;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		LaunchModuleName = "UnrealInsights";
		if (bBuildEditor)
		{
			ExtraModuleNames.Add("EditorStyle");
		}
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bForceBuildTargetPlatforms = true;
		bCompileWithStatsWithoutEngine = true;
		bCompileWithPluginSupport = true;
		bCompileAsioSSLSupport = true;

		// For source code editor access & regex (crossplatform)
		bIncludePluginsForTargetPlatforms = true;
		bCompileICU = true;

		// For UI functionality
		bBuildDeveloperTools = true;

		if (Target.Platform == UnrealTargetPlatform.Win64 && Architecture.bIsX64 && Configuration != UnrealTargetConfiguration.Shipping)
		{
			// UE-274428 - Disabling by default.  LiveCoding starts up Async and can cause ensures when Insights
			// is quickly shutdown
			// bWithLiveCoding = true;
		}

		bHasExports = false;

		// Enable server controls
		GlobalDefinitions.Add("UE_TRACE_SERVER_CONTROLS_ENABLED=1");
		// Have UnrealInsights implicitly launch the trace store.
		GlobalDefinitions.Add("UE_TRACE_SERVER_LAUNCH_ENABLED=1");

		// UE_TRACE_ENABLED
		bEnableTrace = bCmdLineTrace;
		//bEnableTrace = true; // local override
		if (bEnableTrace)
		{
			// Enable memory tracing
			GlobalDefinitions.Add("UE_MEMORY_TRACE_AVAILABLE=1"); // allow memory tracing
			GlobalDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1"); // enable scoped memory tags tracing
			//GlobalDefinitions.Add("UE_CALLSTACK_TRACE_ENABLED=0"); // disable callstack tracing

			// Enable LLM (Low Level Memory Tracker)
			GlobalDefinitions.Add("LLM_ENABLED_IN_CONFIG=1");

			// Disable various tracing systems
			//GlobalDefinitions.Add("LOGTRACE_ENABLED=0");
			//GlobalDefinitions.Add("MISCTRACE_ENABLED=0");
			//GlobalDefinitions.Add("CPUPROFILERTRACE_ENABLED=0");
			//GlobalDefinitions.Add("UE_TRACE_GPU_PROFILER_ENABLED=0");
			//GlobalDefinitions.Add("LOADTIMEPROFILERTRACE_ENABLED=0");
			//GlobalDefinitions.Add("STATSTRACE_ENABLED=0");
			//GlobalDefinitions.Add("COUNTERSTRACE_ENABLED=0");
			//GlobalDefinitions.Add("UE_TASK_TRACE_ENABLED=0");
			//GlobalDefinitions.Add("UE_NET_TRACE_ENABLED=0");
		}

		// Debug for TraceAnalysis module.
		//bTraceAnalysisDebug = true; // local override
		if (bTraceAnalysisDebug)
		{
			// See Engine\Source\Developer\TraceAnalysis\Public\TraceAnalysisDebug.h
			GlobalDefinitions.Add("UE_TRACE_ANALYSIS_DEBUG_API=1");
			GlobalDefinitions.Add("UE_TRACE_ANALYSIS_DEBUG=1");
			GlobalDefinitions.Add("UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL=4");
			GlobalDefinitions.Add("UE_TRACE_ANALYSIS_DEBUG_LEVEL=1");
		}
	}
}
